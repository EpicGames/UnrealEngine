{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Custom Mesh Component",
	"Description" : "A new renderable Component class that allows you to specify custom geometry via C++ or Blueprint.",
	"Category" : "Rendering",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "CustomMeshComponent",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault",
			"PlatformDenyList" :
			[
			]
		}
	]
}
