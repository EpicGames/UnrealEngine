// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/MeshComponent.h"
#include "CustomMeshComponent.generated.h"

#define UE_API CUSTOMMESHCOMPONENT_API

class FPrimitiveSceneProxy;

USTRUCT(BlueprintType)
struct FCustomMeshTriangle
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Triangle)
	FVector Vertex0 = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Triangle)
	FVector Vertex1 = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Triangle)
	FVector Vertex2 = FVector::ZeroVector;
};

/** Component that allows you to specify custom triangle mesh geometry */
UCLASS(MinimalAPI, hidecategories=(Object,LOD, Physics, Collision), editinlinenew, meta=(BlueprintSpawnableComponent), ClassGroup=Rendering)
class UCustomMeshComponent : public UMeshComponent
{
	GENERATED_UCLASS_BODY()

	/** Set the geometry to use on this triangle mesh */
	UFUNCTION(BlueprintCallable, Category="Components|CustomMesh")
	UE_API bool SetCustomMeshTriangles(const TArray<FCustomMeshTriangle>& Triangles);

	/** Add to the geometry to use on this triangle mesh.  This may cause an allocation.  Use SetCustomMeshTriangles() instead when possible to reduce allocations. */
	UFUNCTION(BlueprintCallable, Category = "Components|CustomMesh")
	UE_API void AddCustomMeshTriangles(const TArray<FCustomMeshTriangle>& Triangles);

	/** Removes all geometry from this triangle mesh.  Does not deallocate memory, allowing new geometry to reuse the existing allocation. */
	UFUNCTION(BlueprintCallable, Category = "Components|CustomMesh")
	UE_API void ClearCustomMeshTriangles();

private:

	//~ Begin UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UMeshComponent Interface.
	UE_API virtual int32 GetNumMaterials() const override;
	//~ End UMeshComponent Interface.

	//~ Begin USceneComponent Interface.
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ Begin USceneComponent Interface.

	/** */
	TArray<FCustomMeshTriangle> CustomMeshTris;

	friend class FCustomMeshSceneProxy;
};


#undef UE_API
