// Copyright Epic Games, Inc. All Rights Reserved. 

#include "CustomMeshComponent.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "MaterialDomain.h"
#include "Materials/MaterialRenderProxy.h"
#include "Engine/CollisionProfile.h"
#include "Materials/Material.h"
#include "PrimitiveDrawingUtils.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "DynamicMeshBuilder.h"
#include "Engine/Engine.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "StaticMeshResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomMeshComponent)

/** Scene proxy */
class FCustomMeshSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FCustomMeshSceneProxy(UCustomMeshComponent* Component)
		: FPrimitiveSceneProxy(Component)
		, VertexFactory(GetScene().GetFeatureLevel(), "FCustomMeshSceneProxy")
		, MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform()))
	{
		const FColor VertexColor(255,255,255);

		TArray<FDynamicMeshVertex> Vertices;
		const int32 NumTris = Component->CustomMeshTris.Num();
		Vertices.AddUninitialized(NumTris * 3);
		IndexBuffer.Indices.AddUninitialized(NumTris * 3);
		// Add each triangle to the vertex/index buffer
		for(int32 TriIdx = 0; TriIdx < NumTris; TriIdx++)
		{
			FCustomMeshTriangle& Tri = Component->CustomMeshTris[TriIdx];

			const FVector Edge01 = (Tri.Vertex1 - Tri.Vertex0);
			const FVector Edge02 = (Tri.Vertex2 - Tri.Vertex0);

			const FVector TangentX = Edge01.GetSafeNormal();
			const FVector TangentZ = (Edge02 ^ Edge01).GetSafeNormal();
			const FVector TangentY = (TangentX ^ TangentZ).GetSafeNormal();

			FDynamicMeshVertex Vert;
			
			Vert.Color = VertexColor;
			Vert.SetTangents((FVector3f)TangentX, (FVector3f)TangentY, (FVector3f)TangentZ);

			Vert.Position = (FVector3f)Tri.Vertex0;
			Vertices[TriIdx * 3 + 0] = Vert;
			IndexBuffer.Indices[TriIdx * 3 + 0] = TriIdx * 3 + 0;

			Vert.Position = (FVector3f)Tri.Vertex1;
			Vertices[TriIdx * 3 + 1] = Vert;
			IndexBuffer.Indices[TriIdx * 3 + 1] = TriIdx * 3 + 1;

			Vert.Position = (FVector3f)Tri.Vertex2;
			Vertices[TriIdx * 3 + 2] = Vert;
			IndexBuffer.Indices[TriIdx * 3 + 2] = TriIdx * 3 + 2;
		}

		VertexBuffers.InitFromDynamicVertex(&VertexFactory, Vertices);

		// Enqueue initialization of render resource
		BeginInitResource(&VertexBuffers.PositionVertexBuffer);
		BeginInitResource(&VertexBuffers.StaticMeshVertexBuffer);
		BeginInitResource(&VertexBuffers.ColorVertexBuffer);
		BeginInitResource(&IndexBuffer);
		BeginInitResource(&VertexFactory);

		// Grab material
		Material = Component->GetMaterial(0);
		if(Material == NULL)
		{
			Material = UMaterial::GetDefaultMaterial(MD_Surface);
		}
	}

	virtual ~FCustomMeshSceneProxy()
	{
		VertexBuffers.PositionVertexBuffer.ReleaseResource();
		VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
		VertexBuffers.ColorVertexBuffer.ReleaseResource();
		IndexBuffer.ReleaseResource();
		VertexFactory.ReleaseResource();
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER( STAT_CustomMeshSceneProxy_GetDynamicMeshElements );

		const bool bWireframe = AllowDebugViewmodes() && ViewFamily.EngineShowFlags.Wireframe;

		auto WireframeMaterialInstance = new FColoredMaterialRenderProxy(
			GEngine->WireframeMaterial ? GEngine->WireframeMaterial->GetRenderProxy() : NULL,
			FLinearColor(0, 0.5f, 1.f)
			);

		Collector.RegisterOneFrameMaterialProxy(WireframeMaterialInstance);

		FMaterialRenderProxy* MaterialProxy = NULL;
		if(bWireframe)
		{
			MaterialProxy = WireframeMaterialInstance;
		}
		else
		{
			MaterialProxy = Material->GetRenderProxy();
		}

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];
				// Draw the mesh.
				FMeshBatch& Mesh = Collector.AllocateMesh();
				FMeshBatchElement& BatchElement = Mesh.Elements[0];
				BatchElement.IndexBuffer = &IndexBuffer;
				Mesh.bWireframe = bWireframe;
				Mesh.VertexFactory = &VertexFactory;
				Mesh.MaterialRenderProxy = MaterialProxy;

				FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
				FPrimitiveUniformShaderParametersBuilder Builder;
				BuildUniformShaderParameters(Builder);
				DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), Builder);
				BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

				BatchElement.FirstIndex = 0;
				BatchElement.NumPrimitives = IndexBuffer.Indices.Num() / 3;
				BatchElement.MinVertexIndex = 0;
				BatchElement.MaxVertexIndex = VertexBuffers.PositionVertexBuffer.GetNumVertices() - 1;
				Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
				Mesh.Type = PT_TriangleList;
				Mesh.DepthPriorityGroup = SDPG_World;
				Mesh.bCanApplyViewModeOverrides = false;
				Collector.AddMesh(ViewIndex, Mesh);
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bDynamicRelevance = true;
		Result.bRenderInMainPass = ShouldRenderInMainPass();
		Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
		MaterialRelevance.SetPrimitiveViewRelevance(Result);
		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return !MaterialRelevance.bDisableDepthTest;
	}

	virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }

	uint32 GetAllocatedSize( void ) const { return( FPrimitiveSceneProxy::GetAllocatedSize() ); }

private:

	UMaterialInterface* Material;
	FStaticMeshVertexBuffers VertexBuffers;
	FDynamicMeshIndexBuffer32 IndexBuffer;
	FLocalVertexFactory VertexFactory;

	FMaterialRelevance MaterialRelevance;
};

//////////////////////////////////////////////////////////////////////////

UCustomMeshComponent::UCustomMeshComponent( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	PrimaryComponentTick.bCanEverTick = false;

	SetCollisionProfileName(UCollisionProfile::BlockAllDynamic_ProfileName);
}

bool UCustomMeshComponent::SetCustomMeshTriangles(const TArray<FCustomMeshTriangle>& Triangles)
{
	CustomMeshTris = Triangles;

	// Need to recreate scene proxy to send it over
	MarkRenderStateDirty();
	UpdateBounds();

	return true;
}

void UCustomMeshComponent::AddCustomMeshTriangles(const TArray<FCustomMeshTriangle>& Triangles)
{
	CustomMeshTris.Append(Triangles);

	// Need to recreate scene proxy to send it over
	MarkRenderStateDirty();
	UpdateBounds();
}

void  UCustomMeshComponent::ClearCustomMeshTriangles()
{
	CustomMeshTris.Reset();

	// Need to recreate scene proxy to send it over
	MarkRenderStateDirty();
	UpdateBounds();
}


FPrimitiveSceneProxy* UCustomMeshComponent::CreateSceneProxy()
{
	FPrimitiveSceneProxy* Proxy = NULL;
	if(CustomMeshTris.Num() > 0)
	{
		Proxy = new FCustomMeshSceneProxy(this);
	}
	return Proxy;
}

int32 UCustomMeshComponent::GetNumMaterials() const
{
	return 1;
}


FBoxSphereBounds UCustomMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox BoundingBox(ForceInit);

	// Bounds are tighter if the box is generated from pre-transformed vertices.
	for (int32 Index = 0; Index < CustomMeshTris.Num(); ++Index)
	{
		BoundingBox += LocalToWorld.TransformPosition(CustomMeshTris[Index].Vertex0);
		BoundingBox += LocalToWorld.TransformPosition(CustomMeshTris[Index].Vertex1);
		BoundingBox += LocalToWorld.TransformPosition(CustomMeshTris[Index].Vertex2);
	}

	FBoxSphereBounds NewBounds;
	NewBounds.BoxExtent = BoundingBox.GetExtent();
	NewBounds.Origin = BoundingBox.GetCenter();
	NewBounds.SphereRadius = NewBounds.BoxExtent.Size();

	return NewBounds;
}

