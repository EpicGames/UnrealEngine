// Copyright Epic Games, Inc. All Rights Reserved.

#include "ICustomMeshComponentPlugin.h"




class FCustomMeshComponentPlugin : public ICustomMeshComponentPlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FCustomMeshComponentPlugin, CustomMeshComponent )



void FCustomMeshComponentPlugin::StartupModule()
{
	
}


void FCustomMeshComponentPlugin::ShutdownModule()
{
	
}



